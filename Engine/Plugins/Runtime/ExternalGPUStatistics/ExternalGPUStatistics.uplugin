{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "External GPU Statistics",
	"Description": "Plugin to find out more detailed GPU usage for NVIDIA, AMD, and Intel GPUs.",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux"
	],
	"Modules": [
		{
			"Name": "ExternalGPUStatistics",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit",
			"TargetConfigurationDenyList": [ "Shipping" ],
			"TargetDenyList": ["Server"],
			"PlatformAllowList": [ "Win64", "Linux" ]
		}
	]
}