// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ExternalGPUStatistics : ModuleRules
	{
		public ExternalGPUStatistics(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"RenderCore",
					"RHI",
				}
			);


			bool bWithNVIDIA = (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) && Target.Architecture.bIsX64) || 
								Target.IsInPlatformGroup(UnrealPlatformGroup.Unix);
			if (bWithNVIDIA)
			{
				PrivateDependencyModuleNames.Add("NVML");
				PrivateDefinitions.Add("WITH_VENDOR_NVIDIA=1");
			}
			else
			{
				PrivateDefinitions.Add("WITH_VENDOR_NVIDIA=0");
			}


			bool bWithIntel = (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) && Target.Architecture.bIsX64) || 
								Target.IsInPlatformGroup(UnrealPlatformGroup.Unix);
			if (bWithIntel)
			{
				PrivateDependencyModuleNames.Add("oneAPILevelZero");
				PrivateDefinitions.Add("WITH_VENDOR_INTEL=1");
			}
			else
			{
				PrivateDefinitions.Add("WITH_VENDOR_INTEL=0");
			}

			ShortName = "ExtGPUStats";
		}
	}
}
