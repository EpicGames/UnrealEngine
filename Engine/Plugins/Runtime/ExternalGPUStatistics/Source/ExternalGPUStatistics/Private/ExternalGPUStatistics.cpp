// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "Modules/ModuleManager.h"
#include "IExternalGPUStatistics.h"
#include "RHI.h"
#include "MultiGPU.h"
#include "RenderingThread.h"
#include "Async/AsyncWork.h"
#include "Templates/Atomic.h"
#include "Misc/App.h"
#include "Misc/ScopeLock.h"
#include "TimerManager.h"
#if WITH_VENDOR_NVIDIA
#include "GpuClocker.h"
#endif
#if WITH_VENDOR_INTEL
#include "Vendors/Intel.h"
#endif

namespace
{
static TAutoConsoleVariable<int32> CVarGPUStatisticsEnable(
	TEXT("r.GPUStatistics"), 1,
	TEXT("Whether to enable RHIGetGPUUsage(). (enabled by default)\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarPerProcessGPUStatisticsEnable(
	TEXT("r.GPUStatisticsPerProcess"), 1,
	TEXT("Whether to enable RHIGetGPUUsage() gathering per process information. (disabled by default)\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarGPUStatisticsSkipFrames(
	TEXT("r.GPUStatistics.SkipFrames"), 0,
	TEXT("Number of frame to skip before update the RHIGetGPUUsage()'s cache. (default=0)\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarGPUStatisticsAsync(
	TEXT("r.GPUStatistics.Async"), 1,
	TEXT("Use a separate task to update FD3DBackdoorQueryStatistics to avoid blocking the rendering thread. (enabled by default)\n"),
	ECVF_Default);

struct FGPUStatistics
{
	uint64_t LastSeenTimestamp;
	FRHIGPUUsageFractions UsageFractionsCache;
};

constexpr double NumSecondsToWait = 2.0f;
static double NumSecondsElapsed = 0;
FDelegateHandle DelayedStartHandle;
bool bAttemptedDelayedInit;

#if WITH_VENDOR_NVIDIA
//NVIDIA
static FGpuClocker Clocker;
#endif

// Intel
#if WITH_VENDOR_INTEL
TArray<UE::GPUStats::Intel::FIntelDriver> IntelDrivers;
TArray<UE::GPUStats::Intel::FIntelDevice> IntelDevices;

// One array per GNumExplicitGPUsForRendering
// These two are one to one, the only reason they are separated is the Intel API uses a list of zet_metric_group_handle_t to activate the metrics group
TArray<TArray<zet_metric_group_handle_t>> IntelMetrics;
TArray<TArray<UE::GPUStats::Intel::FIntelMetricsStreamer>> IntelMetricsStreamers;
#endif // WITH_VENDOR_INTEL

FCriticalSection MTQueryStatisticsCriticalSection;
TAtomic<bool> GIsASyncUpdating(false);

int32 GTicksToSkip = 0;
TStaticArray<FGPUStatistics, MAX_NUM_GPUS> GGPUStatistics;
bool bIsPerProcessDataExpectedToWork = false;


bool ShutdownIntel()
{
#if WITH_VENDOR_INTEL
	if (IntelDevices.Num() > 0)
	{
		for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
		{
			IntelMetrics[GPUIndex].Empty();
			for (UE::GPUStats::Intel::FIntelMetricsStreamer & MetricsStreamer : IntelMetricsStreamers[GPUIndex])
			{
				if (MetricsStreamer.MetricStreamerHandle != nullptr)
				{
					UE::GPUStats::Intel::ShutdownMetricStreamer(MetricsStreamer);
				}
			}
			IntelMetricsStreamers[GPUIndex].Empty();
		}
	}
	
	IntelMetrics.Empty();
	IntelMetricsStreamers.Empty();
	IntelDevices.Empty();

	// Only the contexts need to be destroyed;
	for (int32 Index = 0; Index < IntelDrivers.Num(); Index++)
	{
		UE::GPUStats::Intel::FIntelDriver & Driver = IntelDrivers[Index];
		if (Driver.ContextHandle != nullptr)
		{
			UE::GPUStats::Intel::ShutdownDriver(Driver);
		}
	}

	IntelDrivers.Empty();
		
#endif

	return true;
}

bool QueryGlobalGPUStatistics(uint32 GPUIndex, float& OutCurrentGPUUtilization, uint64_t& OutCurrentGPUMemoryUsage, float& OutFrequencyScaling)
{
	bool Result = false;
	OutFrequencyScaling = OutCurrentGPUUtilization = OutCurrentGPUMemoryUsage = 0;
	
	if (IsRHIDeviceNVIDIA())
	{
#if WITH_VENDOR_NVIDIA
		uint32 MaxClock;
		Result = Clocker.GetGpuMaxClock(GPUIndex, MaxClock);
		if (!Result)
		{
			return false;
		}
		
		uint32 GraphicsMHz, MemoryMHz;
		Result = Clocker.GetGpuMHz(GPUIndex, GraphicsMHz, MemoryMHz);
		if (!Result)
		{
			return false;
		}

		OutFrequencyScaling = static_cast<float>(GraphicsMHz)/ static_cast<float>(MaxClock);

		uint32 CurrentGPUUtilization, CurrentGPUMemoryWrites;
		Result = Clocker.GetGpuUsage(GPUIndex, CurrentGPUUtilization, CurrentGPUMemoryWrites);
		if (!Result)
		{
			return false;
		}
		OutCurrentGPUUtilization = CurrentGPUUtilization/100.0f;

		uint64_t TotalMemory, FreeMemory;
		Result = Clocker.GetGpuMemoryUsage(GPUIndex, TotalMemory, FreeMemory, OutCurrentGPUMemoryUsage);

		if (!Result)
		{
			return false;
		}
#endif
	}
	if (IsRHIDeviceIntel())
	{
#if WITH_VENDOR_INTEL
		Result = UE::GPUStats::Intel::CalculateMetrics(IntelDevices[GPUIndex], IntelMetrics[GPUIndex], IntelMetricsStreamers[GPUIndex], OutFrequencyScaling, OutCurrentGPUUtilization, OutCurrentGPUMemoryUsage);
#endif
	}

	return Result;
}

bool QueryCurrentProcessGPUStatistics(uint32 GPUIndex, float& OutCurrentProcessUtilization, uint64_t& OutCurrentProcessMemoryUsage, uint64_t LastSeenTimestamp, uint64_t & OutLastSeenTimestamp)
{
	bool Result = false;

	int32 PID = FPlatformProcess::GetCurrentProcessId();
	if (IsRHIDeviceNVIDIA())
	{
#if WITH_VENDOR_NVIDIA
		uint32 CurrentProcessUtilization;
		uint32 CurrentProcessMemoryWrites;
		// The NVIDIA API sampling may not always get the latest data
		Result = Clocker.GetGpuProcessMHz(GPUIndex, FPlatformProcess::GetCurrentProcessId(), LastSeenTimestamp, CurrentProcessUtilization, CurrentProcessMemoryWrites, OutLastSeenTimestamp);
		if (!Result)
		{
			return false;
		}
		OutCurrentProcessUtilization = CurrentProcessUtilization/100.0f;
		OutCurrentProcessMemoryUsage = CurrentProcessMemoryWrites/100.0f;

		Result = Clocker.GetGpuProcessMemoryUsage(GPUIndex, PID, OutCurrentProcessMemoryUsage);
#endif
	}
	
	return Result;
}

void UpdateGPUUsageCache()
{
	FScopeLock Lock(&MTQueryStatisticsCriticalSection);

	if (!GRHISupportsGPUUsage)
	{
		return;
	}

	const bool bIsPerFrameProcessDataEnabled = CVarPerProcessGPUStatisticsEnable->GetBool();
	
	for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
	{
		// When doing -VMGPU, it's still sharing the same GPU but can display different usages
		// Moreover the GPU usage for each GPU when they are in fact the same ends up with statistics really odd.
		if (GVirtualMGPU && GPUIndex > 0)
		{
			GGPUStatistics[GPUIndex].UsageFractionsCache = GGPUStatistics[0].UsageFractionsCache;
			continue;
		}

		float CurrentGPUMhz, CurrentProcessMHz;
		CurrentGPUMhz = CurrentProcessMHz = 0.0f;
		uint64_t CurrentGPUMemoryUsage, CurrentProcessMemoryUsage;
		CurrentGPUMemoryUsage = CurrentProcessMemoryUsage = 0;
		float GlobalGPUFrequencyScaling = 0;
		bool Result = QueryGlobalGPUStatistics(GPUIndex, CurrentGPUMhz, CurrentGPUMemoryUsage, GlobalGPUFrequencyScaling);
		if (!Result)
		{
			UE_LOGF(LogRHI, Error, "QueryStatistics failed for the entire GPU%d use", GPUIndex);
			continue;
		}

		FRHIGPUUsageFractions NewFractions;
		NewFractions.ClockScaling = GlobalGPUFrequencyScaling;

		NewFractions.ExternalProcessesMHz = CurrentGPUMhz;
		NewFractions.ExternalProcessMemoryUsage = CurrentGPUMemoryUsage;

		if (bIsPerFrameProcessDataEnabled && bIsPerProcessDataExpectedToWork)
		{
			// Save the value here if we need to do comparison at some point
			uint64_t LastSeenTimestamp = GGPUStatistics[GPUIndex].LastSeenTimestamp;
			Result = QueryCurrentProcessGPUStatistics(GPUIndex, CurrentProcessMHz, CurrentProcessMemoryUsage, LastSeenTimestamp, GGPUStatistics[GPUIndex].LastSeenTimestamp);
			if (!Result)
			{
				// This can fail to be sampled if the process is suspended, so we just don't update the per process stats in that case
				continue;
			}
			
			// Because we have the per-process information, we can actually show what the rest of GPU is doing
			NewFractions.ExternalProcessesMHz -= CurrentProcessMHz;
			if (NewFractions.ExternalProcessesMHz < 0.0f)
			{
				NewFractions.ExternalProcessesMHz = 0.0f;
			}
			NewFractions.ExternalProcessMemoryUsage -= CurrentProcessMemoryUsage;
			NewFractions.CurrentProcessMHz = CurrentProcessMHz;
			NewFractions.CurrentProcessMemoryUsage = CurrentProcessMemoryUsage;
		}

		// If we do not expect the per process data to work, then copy the result into the per process data to make the output to the CSV and graphs in Horde make sense
		if (!bIsPerProcessDataExpectedToWork)
		{
			NewFractions.CurrentProcessMHz = NewFractions.ExternalProcessesMHz;
			NewFractions.CurrentProcessMemoryUsage = NewFractions.ExternalProcessMemoryUsage;
		}
		
		GGPUStatistics[GPUIndex].UsageFractionsCache = NewFractions;
	}

	GTicksToSkip = CVarGPUStatisticsSkipFrames.GetValueOnAnyThread();
}

/** An async task used to call tick on the pending update. */
class FUpdateGPUUsageCacheTask : public FNonAbandonableTask
{
public:
	FUpdateGPUUsageCacheTask() = default;

	void DoWork()
	{
		UpdateGPUUsageCache();
		GIsASyncUpdating = false;
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FUpdateGPUUsageCacheTask, STATGROUP_ThreadPoolAsyncTasks);
	}
};

void InitGetGPUUsage();

void CheckForDelayedInit()
{
	if (!bAttemptedDelayedInit)
	{
		if (NumSecondsElapsed < NumSecondsToWait)
		{
			NumSecondsElapsed += FApp::GetDeltaTime();	
		}
		else
		{
			bAttemptedDelayedInit = true;
			FCoreDelegates::OnBeginFrame.Remove(DelayedStartHandle);
			InitGetGPUUsage();
		}
	}
}

void TickGPUUsageCache()
{
	check(IsInRenderingThread());
	
	if (!GRHISupportsGPUUsage)
	{
		return;
	}

	if (GTicksToSkip > 0)
	{
		GTicksToSkip--;
		return;
	}

	if (GIsASyncUpdating)
	{
		// NOP
	}
	else if (CVarGPUStatisticsAsync.GetValueOnRenderThread())
	{
		GIsASyncUpdating = true;
		(new FAutoDeleteAsyncTask<FUpdateGPUUsageCacheTask>())->StartBackgroundTask();
	}
	else
	{
		UpdateGPUUsageCache();
	}
}

FRHIGPUUsageFractions GetGPUUsage(uint32 GPUIndex)
{
	check(GRHISupportsGPUUsage);
	check(GPUIndex < GNumExplicitGPUsForRendering);
	check(IsInGameThread() || IsInRenderingThread());
	return GGPUStatistics[GPUIndex].UsageFractionsCache;
}

void InitGetGPUUsage()
{
	check(IsInGameThread());
	FlushRenderingCommands();

	check(GDynamicRHI);

	// Requires an actual RHI.
	if (GDynamicRHI->GetInterfaceType() == ERHIInterfaceType::Null)
	{
		UE_LOGF(LogRHI, Display, "FExternalGPUStatistics disabled due to null RHI.");
		return;
	}

	if (IsRunningCommandlet())
	{
		UE_LOGF(LogRHI, Display, "FExternalGPUStatistics doesn't matter in any commandlets.");
		return;
	}

	static bool bAllowSoftwareRendering = FParse::Param(FCommandLine::Get(), TEXT("AllowSoftwareRendering"));
	if (bAllowSoftwareRendering)
	{
		UE_LOGF(LogRHI, Display, "FExternalGPUStatistics is not supported with software rendering.");
		return;
	}

	// NVIDIA and AMD does have per process information, Intel does not
	// We need to also output the metrics to different fields
	
	if (IsRHIDeviceNVIDIA())
	{
		for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
		{
			// Nothing to do because the GpuClocker class handles the work.
		}
		bIsPerProcessDataExpectedToWork = true;
	}
	else if (IsRHIDeviceAMD())
	{
		for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
		{
			// Setup AMD-SMI
		}
	}
	else if (IsRHIDeviceIntel())
	{
#if WITH_VENDOR_INTEL
		using namespace UE::GPUStats::Intel;

		// Get all of our drivers
		if (!GetDrivers(IntelDrivers))
		{
			UE_LOGF(LogRHI, Error, "Failed to initialize Intel Level Zero API!");
			ShutdownIntel();
			return;
		}

		// Accumulate a list of all the GPUs in the Driver
		for (int32 Index = 0; Index < IntelDrivers.Num(); Index++)
		{
			if (!GetDevicesForDriver(IntelDrivers[Index],IntelDevices))
			{
				UE_LOGF(LogRHI, Error, "Failed to initialize devices for driver %u!", Index);
				ShutdownIntel();
				return;
			}
		}
		
		if (GNumExplicitGPUsForRendering > static_cast<uint32>(IntelDevices.Num()))
		{
			UE_LOGF(LogRHI, Error, "Discovered %u GPUs that can render, expected at least %u", IntelDevices.Num(), GNumExplicitGPUsForRendering);
			ShutdownIntel();
			return;
		}

		// An array per Device that has metrics and their associated streamers that we are going to read per tick
		IntelMetrics.Reserve(GNumExplicitGPUsForRendering);
		IntelMetricsStreamers.Reserve(GNumExplicitGPUsForRendering);
		
		// Iterate over the Devices we just discovered and set up
		for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
		{
			FIntelDevice & Device = IntelDevices[GPUIndex];
			IntelMetrics.Add(TArray<zet_metric_group_handle_t>());
			IntelMetricsStreamers.Add(TArray<FIntelMetricsStreamer>());
			
			if (!SetupMetricsForDevice(Device, IntelMetrics[GPUIndex]))
			{
				UE_LOGF(LogRHI, Error, "Failed to setup metrics for GPU %u", GPUIndex);
				ShutdownIntel();
				return;
			}
			
			if(zetContextActivateMetricGroups(Device.Driver.ContextHandle, Device.DeviceHandle, IntelMetrics[GPUIndex].Num(), IntelMetrics[GPUIndex].GetData()) != ZE_RESULT_SUCCESS)
			{
				UE_LOGF(LogRHI,Error, "Could not activate metrics for GPU %u", GPUIndex);
				ShutdownIntel();
				return;
			}

			// Metrics Streamers should be 1:1 with the Metrics Groups
			IntelMetricsStreamers[GPUIndex].SetNum(IntelMetrics[GPUIndex].Num());
			
			if (!SetupMetricsStreamersForDevice(Device, IntelMetrics[GPUIndex], IntelMetricsStreamers[GPUIndex]))
			{
				UE_LOGF(LogRHI, Error, "Failed to setup metrics streamers for GPU %u", GPUIndex);
				ShutdownIntel();
				return;
			}
		}
#endif // WITH_VENDOR_INTEL
	}
	else
	{
		UE_LOGF(LogRHI, Error, "FExternalGPUStatistics could not be initialized on unsupported vendor!");
		return;
	}

	FScopeLock Lock(&MTQueryStatisticsCriticalSection);

	// Verify both GPU statistic functions works.
	for (uint32 GPUIndex = 0; GPUIndex < GNumExplicitGPUsForRendering; GPUIndex++)
	{
		float CurrentGPUMHz, CurrentProcessMHz;
		uint64_t CurrentGPUMemoryUsage, CurrentProcessMemoryUsage;
		float GPUFrequencyScaling;
		if (!QueryGlobalGPUStatistics(GPUIndex, CurrentGPUMHz, CurrentGPUMemoryUsage, GPUFrequencyScaling))
		{
			UE_LOGF(LogRHI, Error, "QueryStatistics failed for the entire GPU%d use", GPUIndex);
			return;
		}

		// Some APIs cannot get per process data or that data may be equivalent to the system data
		if (CVarPerProcessGPUStatisticsEnable.GetValueOnAnyThread() && bIsPerProcessDataExpectedToWork)
		{
			uint64_t LasSeenTimeStamp = 0;
			if (!QueryCurrentProcessGPUStatistics(GPUIndex, CurrentProcessMHz, CurrentProcessMemoryUsage, 0, LasSeenTimeStamp))
			{
				UE_LOGF(LogRHI, Error, "QueryStatistics failed for the current process use of GPU%d", GPUIndex);
				return;
			}	
		}
	}

	RHIGetGPUUsage = &GetGPUUsage;
	GRHISupportsGPUUsage = true;
	UE_LOGF(LogRHI, Display, "ExternalGPUStatistics module fully operational");
} // InitGetGPUUsage()


void ShutdownGetGPUUsage()
{
	check(IsInGameThread());
	if (!GRHISupportsGPUUsage)
	{
		return;
	}
	
	if (IsRHIDeviceIntel())
	{
#if WITH_VENDOR_INTEL
		ShutdownIntel();
#endif
	}

	FlushRenderingCommands();

	FScopeLock Lock(&MTQueryStatisticsCriticalSection);
	
	GRHISupportsGPUUsage = false;
	RHIGetGPUUsage = nullptr;
	UE_LOGF(LogRHI, Display, "ExternalGPUStatistics module is no longer operational");
}

void OnChangeGPUStatisticsEnable(IConsoleVariable* CVar)
{
	check(IsInGameThread());
	bool bDesireEnabled = CVar->GetInt() != 0;

	if (bDesireEnabled != GRHISupportsGPUUsage)
	{
		if (bDesireEnabled)
		{
			InitGetGPUUsage();
		}
		else
		{
			ShutdownGetGPUUsage();
		}
	}
}

void InitExternalGPUStatisticsModule()
{
	if (CVarGPUStatisticsEnable.GetValueOnGameThread() != 0)
	{
		// Need to delay this by a couple seconds because the API can take a bit to populate
		bAttemptedDelayedInit = false;
	}
	else
	{
		UE_LOGF(LogRHI, Display, "FExternalGPUStatistics has been disabled with r.GPUStatistics=0");
	}

	DelayedStartHandle = FCoreDelegates::OnBeginFrame.AddStatic(CheckForDelayedInit);
	FCoreDelegates::OnBeginFrameRT.AddStatic(TickGPUUsageCache);
	CVarGPUStatisticsEnable.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeGPUStatisticsEnable));
}

} // namespace

class FExternalGPUStatistics : public IExternalGPUStatistics
{
	/** IModuleInterface implementation */
#if WITH_EDITOR
	virtual void StartupModule() override
	{
		InitExternalGPUStatisticsModule();
	}

	virtual void ShutdownModule() override
	{
		
	}
#endif
};

IMPLEMENT_MODULE( FExternalGPUStatistics, ExternalGPUStatistics )

// Works around a bug where the module is compiled and linked into staged build, but not initializing because not in the .upluginmanifest
#if !WITH_EDITOR
class FExternalGPUStatisticsModuleInit
{
public:
	FExternalGPUStatisticsModuleInit()
	{
		FCoreDelegates::GetOnPostEngineInit().AddStatic(&InitExternalGPUStatisticsModule);
	}
};

FExternalGPUStatisticsModuleInit GExternalGPUStatisticsModuleInit;
#endif
