// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_VENDOR_INTEL
#include "Intel.h"
#include "RHI.h"
#include "MultiGPU.h"

//Intel
#define INTEL_RETURN_IF_RESULT_DOES_MATCH(Result, Expected, FormatString, ...)\
do\
{\
	if (Result != Expected)\
	{\
		UE_LOG(LogRHI, Error, TEXT(FormatString), ##__VA_ARGS__);\
		return false;\
	}\
}\
while(0)

#define INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, FormatString, ...) INTEL_RETURN_IF_RESULT_DOES_MATCH(Result, ZE_RESULT_SUCCESS, FormatString, ##__VA_ARGS__)

namespace UE::GPUStats::Intel
{
const char* RENDER_BASIC = "RenderBasic";

// Must match the above in terms of order and number
// This assumes that the naming is consistent across all hardware revisions (see https://github.com/intel/metrics-discovery/tree/master/docs for the list of possible strings)
// If not then we may have to multiple lists achieving this effect
const char* INTEL_METRICS_STRINGS[MAX_METRICS] =
{
	"AvgGpuCoreFrequencyMHz",
	"GpuBusy",
	"SlmBytesWritten"
};

bool GetDrivers(TArray<FIntelDriver> & Drivers)
{
	uint32 DriverCount = 0;
		
	ze_init_driver_type_desc_t DriverTypeDesc = {};
	DriverTypeDesc.stype = ZE_STRUCTURE_TYPE_INIT_DRIVER_TYPE_DESC;
	DriverTypeDesc.flags = ZE_INIT_DRIVER_TYPE_FLAG_GPU;
	DriverTypeDesc.pNext = nullptr;

	// Required according to the Intel docs to enable the metrics features
	FPlatformMisc::SetEnvironmentVar(TEXT("ZET_ENABLE_METRICS"), TEXT("1"));
	// Get the number of the drivers
	ze_result_t Result = zeInitDrivers(&DriverCount, nullptr, &DriverTypeDesc);
	INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not initialize Level Zero!");

	// Now get the actual driver objects
	TArray<ze_driver_handle_t> TempDrivers;
	TempDrivers.SetNum(DriverCount);
	Result = zeInitDrivers(&DriverCount, TempDrivers.GetData(), &DriverTypeDesc);
	INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not get drivers from Level Zero!");

	// Most systems should only have one for GPUs
	// Build our devices and contexts
	for (int32 DriverIndex = 0; DriverIndex < TempDrivers.Num(); DriverIndex++)
	{
		ze_driver_handle_t Driver = TempDrivers[DriverIndex];
		ze_context_desc_t ContextDesc = {};
		ContextDesc.stype = ZE_STRUCTURE_TYPE_CONTEXT_DESC;
		ContextDesc.pNext = nullptr;
		zet_context_handle_t Context;

		Result = zeContextCreate(Driver, &ContextDesc, &Context);
		INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not create context for driver index: %u!", DriverIndex);

		FIntelDriver Info = {};
		Info.DriverHandle = Driver;
		Info.ContextHandle = Context;

		Drivers.Add(Info);
	}

	return true;
}

bool GetDevicesForDriver(const FIntelDriver Driver, TArray<FIntelDevice> & Devices)
{
	uint32_t DeviceCount = 0;
	// Get the number of devices
	ze_result_t Result = zeDeviceGet(Driver.DriverHandle, &DeviceCount, nullptr);
	INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not get number of devices for driver!");

	TArray<ze_device_handle_t> TempDevices;
	TempDevices.SetNum(DeviceCount);

	Result = zeDeviceGet(Driver.DriverHandle, &DeviceCount, TempDevices.GetData());
	INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not get handles to devices for driver!");

	for (uint32 DeviceIndex = 0; DeviceIndex < DeviceCount; DeviceIndex++)
	{
		ze_device_handle_t Device = TempDevices[DeviceIndex];
		ze_device_properties_t DeviceProperties = {};
		DeviceProperties.stype = ZE_STRUCTURE_TYPE_DEVICE_PROPERTIES;
		Result = zeDeviceGetProperties(Device, &DeviceProperties);
		INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not get properties of device %u for driver!", DeviceIndex);

		if (Device && DeviceProperties.type == ZE_DEVICE_TYPE_GPU)
		{
			FIntelDevice IntelDeviceInfo = {};
			IntelDeviceInfo.Driver = Driver;
			IntelDeviceInfo.DeviceHandle = Device;
			IntelDeviceInfo.CoreClockRate = DeviceProperties.coreClockRate;
			Devices.Add(IntelDeviceInfo);
		}
	}

	return true;
}

bool SetupMetricsForDevice(const FIntelDevice & Device, TArray<zet_metric_group_handle_t> & MetricsForDevice)
{
	constexpr zet_metric_group_sampling_type_flag_t SamplingType = ZET_METRIC_GROUP_SAMPLING_TYPE_FLAG_TIME_BASED;
	// Obtain available metric groups for the specific device
	uint32 MetricGroupCount = 0;
	ze_result_t Result = zetMetricGroupGet(Device.DeviceHandle, &MetricGroupCount, nullptr);
	INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not get number of metrics for device!");

	TArray<zet_metric_group_handle_t> MetricGroups;
	MetricGroups.SetNum(MetricGroupCount);
		
	Result = zetMetricGroupGet(Device.DeviceHandle, &MetricGroupCount, MetricGroups.GetData());
	INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not get metrics group handles for device!");

	// Iterate over all metric groups available
	for(uint32 Index = 0; Index < MetricGroupCount; Index++)
	{
		// Get metric group and its properties
		zet_metric_group_properties_t MetricGroupProperties {};
		MetricGroupProperties.stype = ZET_STRUCTURE_TYPE_METRIC_GROUP_PROPERTIES;
		Result = zetMetricGroupGetProperties(MetricGroups[Index], &MetricGroupProperties);
		INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not get metrics group properties for index %u on device!", Index);

		FString MetricGroupName = StringCast<TCHAR>(MetricGroupProperties.name).Get();

		UE_LOGF(LogRHI, Log, "Found metric group \"%ls\" for device", *MetricGroupName);

		// RenderBasic information for what we need for now
		// We can always expand this out as necessary
		if((MetricGroupProperties.samplingType & SamplingType) == SamplingType && (strcmp(RENDER_BASIC, MetricGroupProperties.name) == 0))
		{
			// If we have more than one group here, we will need to check the domain of the already selected groups
			// Two metrics of the same domain may not be active at the same time
			// I cannot find any examples of what domains for particular metrics are
			// Grab the group for now, we will crack it open when querying from the streamer later
			MetricsForDevice.Add(MetricGroups[Index]);
			break;
		}
	}
	
	return true;
}

bool SetupMetricsStreamersForDevice(const FIntelDevice & Device, TArray<zet_metric_group_handle_t> & MetricsForDevice, TArray<FIntelMetricsStreamer> & MetricsStreamersForDevice)
{
		zet_metric_streamer_desc_t StreamerDesc = {};
		StreamerDesc.samplingPeriod = 1000000; // Every millisecond by default
		StreamerDesc.notifyEveryNReports = 32768; // This number is arbitrary because we are not actually going to send the notification event
		StreamerDesc.pNext = nullptr;

		// Init our streamers so we can grab data the next clock tick
		for (int32 GroupIndex = 0; GroupIndex < MetricsForDevice.Num(); GroupIndex++)
		{
			zet_metric_group_handle_t MetricGroup = MetricsForDevice[GroupIndex];
			zet_metric_streamer_handle_t Streamer = nullptr;
			ze_result_t Result = zetMetricStreamerOpen(Device.Driver.ContextHandle, Device.DeviceHandle, MetricGroup, &StreamerDesc, nullptr, &Streamer);
			INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not open metrics streamer for metrics group %u on device!", GroupIndex);

			uint32 NumMetricsInGroup = 0;
			Result = zetMetricGet(MetricGroup, &NumMetricsInGroup, nullptr);
			INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not get number of metrics in group %u on device!", GroupIndex);
				
			FIntelMetricsStreamer & MetricsStreamer = MetricsStreamersForDevice[GroupIndex];
			MetricsStreamer.MetricStreamerHandle = Streamer;
			MetricsStreamer.NumMetrics = NumMetricsInGroup;


			TArray<zet_metric_handle_t> MetricsInGroup;
			MetricsInGroup.SetNum(NumMetricsInGroup);
			Result = zetMetricGet(MetricGroup, &NumMetricsInGroup, MetricsInGroup.GetData());
			INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not get all metrics in group %u on device!", GroupIndex);

			// Mark the indices as invalid until we have found them all
			for (uint8 MetricToLookForIndex = 0; MetricToLookForIndex < MAX_METRICS; MetricToLookForIndex++)
			{
				MetricsStreamer.MetricIndices[MetricToLookForIndex] = -1;
			}

			// Now search the metrics for the strings we are interested in
			uint32 NumMetricsToLookFor = MAX_METRICS;

			for (uint32 MetricIndex = 0; MetricIndex < NumMetricsInGroup; MetricIndex++)
			{
				zet_metric_properties_t MetricProperties = {};
				Result = zetMetricGetProperties(MetricsInGroup[MetricIndex], &MetricProperties);
				INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Could not get properties for metric %u in group %u on device!", MetricIndex, GroupIndex);

				UE_LOGF(LogRHI, Log, "Found metric property: %ls", ANSI_TO_TCHAR(MetricProperties.name));
				// Loop over all the metrics we need to function
				// In theory, we could be OK without some of these but it's easy to make it generic for now
				for (uint8 MetricToLookForIndex = 0; MetricToLookForIndex < MAX_METRICS; MetricToLookForIndex++)
				{
					if (strncmp(MetricProperties.name, INTEL_METRICS_STRINGS[MetricToLookForIndex], strlen(INTEL_METRICS_STRINGS[MetricToLookForIndex])) == 0)
					{
						NumMetricsToLookFor--;
						MetricsStreamer.MetricIndices[MetricToLookForIndex] = MetricIndex;
						break;
					}
				}

				// Found everything, leave the loop
				if (NumMetricsToLookFor == 0)
				{
					break;
				}
			}

			if (NumMetricsToLookFor > 0)
			{
				UE_LOGF(LogRHI, Log, "Could not find all metrics required for gathering data in metrics group %u on device!", GroupIndex);
				return true;
			}
		}
	
	return true;
}

bool CalculateMetrics(const FIntelDevice & Device, const TArray<zet_metric_group_handle_t> & Metrics, TArray<FIntelMetricsStreamer> & MetricsStreamers, float & OutFrequencyScaling, float & OutCurrentGPUUtilization, uint64_t & OutCurrentGPUMemoryUsage)
{
	for (int32 Index = 0; Index < Metrics.Num(); Index++)
	{
		ze_result_t IntelResult;

		zet_metric_group_handle_t MetricGroup = Metrics[Index];
		FIntelMetricsStreamer & MetricsStreamer = MetricsStreamers[Index];
		
		size_t RawDataSize = 0;

		IntelResult = zetMetricStreamerReadData(MetricsStreamer.MetricStreamerHandle, UINT32_MAX, &RawDataSize, nullptr);

		if (IntelResult == ZE_RESULT_NOT_READY || RawDataSize == 0)
		{
			return true;
		}

		if (IntelResult != ZE_RESULT_SUCCESS)
		{
			UE_LOGF(LogRHI, Error, "Failed to get size of raw data for streamer %u!", Index);
			return false;
		}
		MetricsStreamer.RawData.SetNum(RawDataSize);

		IntelResult = zetMetricStreamerReadData(MetricsStreamer.MetricStreamerHandle, RawDataSize, &RawDataSize, MetricsStreamer.RawData.GetData());
		if (IntelResult == ZE_RESULT_NOT_READY)
		{
			return true;
		}

		if (IntelResult != ZE_RESULT_SUCCESS)
		{
			UE_LOGF(LogRHI, Error, "Failed to get actual raw data for streamer %u!", Index);
			return false;
		}

		uint32 NumMetrics = 0;
		IntelResult = zetMetricGroupCalculateMetricValues(MetricGroup, ZET_METRIC_GROUP_CALCULATION_TYPE_METRIC_VALUES, RawDataSize, MetricsStreamer.RawData.GetData(), &NumMetrics, nullptr);
		if (IntelResult == ZE_RESULT_NOT_READY)
		{
			return true;
		}

		if (IntelResult != ZE_RESULT_SUCCESS)
		{
			UE_LOGF(LogRHI, Error, "Failed to get number of calculated metrics for streamer %u!", Index);
			return false;
		}
		
		MetricsStreamer.CollectedMetrics.SetNum(NumMetrics);
		IntelResult = zetMetricGroupCalculateMetricValues(MetricGroup, ZET_METRIC_GROUP_CALCULATION_TYPE_METRIC_VALUES, RawDataSize, MetricsStreamer.RawData.GetData(), &NumMetrics, MetricsStreamer.CollectedMetrics.GetData());
		if (IntelResult == ZE_RESULT_NOT_READY)
		{
			return true;
		}

		if (IntelResult != ZE_RESULT_SUCCESS)
		{
			UE_LOGF(LogRHI, Error, "Failed to get calculated metrics for index %u", Index);
			return false;
		}

		uint64 AverageClockFrequencyAccumulated = 0;
		float UtilizationAccumulated = 0.0f;
		
		// Metrics are always collected in groups, so we need to iterate per so-called report
		const uint32 NumReports = NumMetrics / MetricsStreamer.NumMetrics;
		for( uint32 Report = 0; Report < NumReports; Report++ )
		{
			for( uint32 Metric = 0; Metric < MetricsStreamer.NumMetrics; Metric++ )
			{
				zet_typed_value_t Data = MetricsStreamer.CollectedMetrics[Report * MetricsStreamer.NumMetrics + Metric];
				if (Metric == MetricsStreamer.MetricIndices[AvgGpuCoreFrequencyMHz])
				{
					// Currently assuming integer type because that's what the core clock rate of the device is stored as
					switch (Data.type)
					{
						case ZET_VALUE_TYPE_UINT64:
						{
							AverageClockFrequencyAccumulated += Data.value.ui64;
							break;
						}
						default:
							break;
					}
				}
				else if (Metric == MetricsStreamer.MetricIndices[GpuBusy])
				{
					switch (Data.type)
					{
						case ZET_VALUE_TYPE_FLOAT32:
							{
								UtilizationAccumulated += Data.value.fp32;
								break;
							}
						default:
							break;
					}
				}
				else if (Metric == MetricsStreamer.MetricIndices[SlmBytesWritten])
				{
					// No need to accumulate on this one, just take the most recent
					// Assuming that we specify this in bytes
					switch (Data.type)
					{
					case ZET_VALUE_TYPE_UINT32:
						{
							OutCurrentGPUMemoryUsage = Data.value.ui32;
							break;
						}
					case ZET_VALUE_TYPE_UINT64:
						{
							OutCurrentGPUMemoryUsage = Data.value.ui64;
							break;
						}
					default:
						break;
					}
					
				}
			}
		}
		
		float AverageClockFrequency = static_cast<float>(AverageClockFrequencyAccumulated) / NumReports;
		OutFrequencyScaling = AverageClockFrequency / Device.CoreClockRate;
		// Unknown if this is fractional or percentage, so a division may need to occur
		OutCurrentGPUUtilization = UtilizationAccumulated / (100*NumReports);
	}

	return true;
}

bool ShutdownMetricStreamer(FIntelMetricsStreamer & MetricsStreamer)
{
	if (MetricsStreamer.MetricStreamerHandle != nullptr)
	{
		// No error checking here because all error codes are the base ones, no special cases to cover
		zetMetricStreamerClose(MetricsStreamer.MetricStreamerHandle);	
	}
	
	return true;
}

bool ShutdownDriver(FIntelDriver & Driver)
{
	if (Driver.ContextHandle != nullptr)
	{
		ze_result_t Result = zeContextDestroy(Driver.ContextHandle);
		INTEL_RETURN_FALSE_IF_NOT_SUCCESS(Result, "Failed to destroy driver!");
		Driver.ContextHandle = nullptr;
		Driver.DriverHandle = nullptr;
	}
	
	return true;
}

};
#endif //WITH_VENDOR_INTEL
