// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_VENDOR_INTEL
#include "CoreMinimal.h"
THIRD_PARTY_INCLUDES_START
#include "ze_api.h"
#include "zet_api.h"
THIRD_PARTY_INCLUDES_END

namespace UE::GPUStats::Intel
{

// Bundle these two together because it is only permissible to have one context per driver
// And as we shut down we want to destroy both of these
struct FIntelDriver
{
	ze_driver_handle_t DriverHandle = nullptr;
	ze_context_handle_t ContextHandle = nullptr;
};

struct FIntelDevice
{
	// These are pointers, so it should be fine to just duplicate these
	// Also should avoid an additional cache miss
	FIntelDriver Driver;
	ze_device_handle_t DeviceHandle = nullptr;
	// Used to determine our clock frequency percentage
	uint32 CoreClockRate = 0;
};

// These indices are not tied to the API but are just a way to index into the string and not have a dynamic array in the Metrics Streamer
enum EIntelMetricsToLookFor
{
	AvgGpuCoreFrequencyMHz = 0,
	GpuBusy = 1,
	SlmBytesWritten = 2,
	MAX_METRICS = 3
};

struct FIntelMetricsStreamer
{
	zet_metric_streamer_handle_t MetricStreamerHandle = nullptr;
	// Used for iteration over the individual reports, which are done at the group level
	uint32 NumMetrics = 0;
	int32 MetricIndices[MAX_METRICS]; // These are the indices at which we should read from the metrics report
	// The Metrics Streamer provides data in a raw format of bytes which is then consumed by a separate function
	TArray<uint8> RawData;
	// The properly typed data for user consumption
	TArray<zet_typed_value_t> CollectedMetrics;
};

bool GetDrivers(TArray<FIntelDriver> & Drivers);
bool GetDevicesForDriver(const FIntelDriver Driver, TArray<FIntelDevice> & Devices);
bool SetupMetricsForDevice(const FIntelDevice & Device, TArray<zet_metric_group_handle_t> & MetricsForDevice);
bool SetupMetricsStreamersForDevice(const FIntelDevice & Device, TArray<zet_metric_group_handle_t> & MetricsForDevice, TArray<FIntelMetricsStreamer> & MetricsStreamersForDevice);
bool CalculateMetrics(const FIntelDevice & Device, const TArray<zet_metric_group_handle_t> & Metrics, TArray<FIntelMetricsStreamer> & MetricsStreamers, float & OutFrequencyScaling, float & OutCurrentGPUUtilization, uint64_t & OutCurrentGPUMemoryUsage);
// Only these two have Level Zero API functions that special attention, the Metric Streamer may not even need it
bool ShutdownMetricStreamer(FIntelMetricsStreamer & MetricsStreamer);
bool ShutdownDriver(FIntelDriver & Driver);

};
#endif // WITH_VENDOR_INTEL
